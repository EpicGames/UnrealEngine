{
    "FileVersion": 3,
    "Version": 1,
    "VersionName": "1.0",
    "FriendlyName": "Live Link Hub OBS Device",
    "Description": "Live Link Hub device for controlling OBS Studio recording and ingesting recorded video as mono takes via the OBS WebSocket v5 protocol.",
    "Category": "Virtual Production",
    "CreatedBy": "Epic Games, Inc.",
    "CreatedByURL": "https://epicgames.com",
    "CanContainContent": false,
    "IsBetaVersion": false,
    "IsExperimentalVersion": true,
    "Installed": false,
    "Modules": [
        {
            "Name": "LiveLinkOBSDevice",
            "Type": "Editor",
            "LoadingPhase": "Default"
        }
    ],
    "Plugins": [
        {
            "Name": "CaptureManagerApp",
            "Enabled": true
        },
        {
            "Name": "CaptureManagerDevices",
            "Enabled": true
        },
        {
            "Name": "LiveLinkDevice",
            "Enabled": true
        },
        {
            "Name": "LiveLinkHub",
            "Enabled": true
        }
    ]
}
