// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkOBSDevice : ModuleRules
{
    public LiveLinkOBSDevice(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CaptureManagerSettings",
                "CaptureManagerTakeMetadata",
                "CaptureUtils",
                "Core",
                "CoreUObject",
                "Engine",
                "IngestLiveLinkDevice",
                "Json",
                "LiveLinkCapabilities",
                "LiveLinkDevice",
                "LiveLinkHub",
                "LiveLinkInterface",
                "NamingTokens",
                "NamingTokensUI",
                "PropertyEditor",
                "Slate",
                "SlateCore",
                "VideoLiveLinkDeviceCommon",
                "WebSockets",
            }
        );

        if (Target.Platform == UnrealTargetPlatform.Win64
         || Target.Platform == UnrealTargetPlatform.Mac
         || Target.Platform == UnrealTargetPlatform.Linux)
        {
            AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
            PublicDefinitions.Add("LLOBS_WITH_OPENSSL=1");
        }
        else
        {
            PublicDefinitions.Add("LLOBS_WITH_OPENSSL=0");
        }

        CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
    }
}
