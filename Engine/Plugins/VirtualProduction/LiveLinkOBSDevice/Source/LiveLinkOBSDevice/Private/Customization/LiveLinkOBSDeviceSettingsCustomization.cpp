// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOBSDeviceSettingsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Devices/LiveLinkOBSDevice.h"
#include "IDetailPropertyRow.h"
#include "NamingTokensSpecifiers.h"
#include "SNamingTokensEditableTextBox.h"

#define LOCTEXT_NAMESPACE "LiveLinkOBSDevice"


TSharedRef<IDetailCustomization> FLiveLinkOBSDeviceSettingsCustomization::MakeInstance()
{
    return MakeShared<FLiveLinkOBSDeviceSettingsCustomization>();
}

void FLiveLinkOBSDeviceSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
    IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("OBS Device");

    TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
    Category.GetDefaultProperties(DefaultProperties);

    // Namespace filter: restrict autocomplete to LLH tokens only.
    FNamingTokenFilterArgs FilterArgs;
    FilterArgs.bIncludeGlobal = false;
    FilterArgs.AdditionalNamespacesToInclude.Add(TEXT("llh"));

    // Hide all properties first. Without this, any explicit AddProperty call places
    // the row before the auto-generated block, putting FilenameFormat above DisplayName.
    for (const TSharedRef<IPropertyHandle>& PropertyHandle : DefaultProperties)
    {
        DetailBuilder.HideProperty(PropertyHandle);
    }

    // Re-add all properties in their original declaration order.
    // FilenameFormat (tagged meta=(NamingTokens)) gets the custom autocomplete widget.
    // All other properties are re-added with their default widgets unchanged.
    for (TSharedRef<IPropertyHandle>& PropertyHandle : DefaultProperties)
    {
        IDetailPropertyRow& Row = Category.AddProperty(PropertyHandle);

        if (!PropertyHandle->HasMetaData(*UE::NamingTokens::Specifiers::UseNamingTokens))
        {
            continue;
        }

        Row.CustomWidget()
            .NameContent()
            [
                PropertyHandle->CreatePropertyNameWidget(
                    LOCTEXT("FilenameFormatLabel", "Output Filename Format"))
            ]
            .ValueContent()
            [
                SNew(SNamingTokensEditableTextBox)
                // Show the raw token string; the user is editing a format template,
                // not viewing a resolved filename.
                .ShouldEvaluateTokens(false)
                .FilterArgs(FilterArgs)
                .NamespaceSuggestionPriority(TArray<FString>{ TEXT("llh") })
                .Text_Lambda([PropertyHandle]() -> FText
                {
                    FString Value;
                    PropertyHandle->GetValue(Value);
                    return FText::FromString(Value);
                })
                .OnTextChanged_Lambda([PropertyHandle](const FText& InText)
                {
                    PropertyHandle->SetValue(InText.ToString());
                })
            ];
    }
}

#undef LOCTEXT_NAMESPACE
