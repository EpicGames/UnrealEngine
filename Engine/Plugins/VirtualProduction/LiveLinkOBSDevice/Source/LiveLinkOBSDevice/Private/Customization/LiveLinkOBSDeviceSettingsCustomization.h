// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;


class FLiveLinkOBSDeviceSettingsCustomization : public IDetailCustomization
{
public:
    static TSharedRef<IDetailCustomization> MakeInstance();

    //~ Begin IDetailCustomization interface
    virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
    //~ End IDetailCustomization interface
};
