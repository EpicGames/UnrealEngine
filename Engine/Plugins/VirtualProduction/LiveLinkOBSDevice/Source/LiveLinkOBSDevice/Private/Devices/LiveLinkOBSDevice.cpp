// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOBSDevice.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "IWebSocket.h"

#if LLOBS_WITH_OPENSSL
#include <openssl/sha.h>
#endif
#include "CaptureManagerTakeMetadata.h"
#include "Ingest/IngestCapability_Events.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "Logging/StructuredLog.h"
#include "Misc/Base64.h"
#include "NamingTokensEngineSubsystem.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Utils/TakeDiscoveryExpressionParser.h"
#include "Utils/VideoDeviceThumbnailExtractor.h"
#include "WebSocketsModule.h"


DEFINE_LOG_CATEGORY(LogLiveLinkOBSDevice);

#define LOCTEXT_NAMESPACE "LiveLinkOBSDevice"


// -----------------------------------------------------------------------
// OBS WebSocket v5 op-codes
// https://github.com/obsproject/obs-websocket/blob/master/docs/generated/protocol.md
// -----------------------------------------------------------------------
namespace OBSWebSocketOpCode
{
    static constexpr int32 Hello      = 0;  // OBS -> Client
    static constexpr int32 Identify   = 1;  // Client -> OBS
    static constexpr int32 Identified = 2;  // OBS -> Client
    static constexpr int32 Event      = 5;  // OBS -> Client
    static constexpr int32 Request    = 6;  // Client -> OBS
    static constexpr int32 Response   = 7;  // OBS -> Client
}

namespace OBSSupportedVideoFormats
{
    static const TArray<FStringView> Extensions =
    {
        TEXTVIEW("mp4"),
        TEXTVIEW("mkv"),
        TEXTVIEW("mov"),
    };

    static const TArray<FString::ElementType> Delimiters =
    {
        '-',
        '_',
        '.',
    };
}


// -----------------------------------------------------------------------
// ULiveLinkDevice interface
// -----------------------------------------------------------------------
EDeviceHealth ULiveLinkOBSDevice::GetDeviceHealth() const
{
    if (bAuthFailed)
    {
        return EDeviceHealth::Error;
    }
    if (bPasswordRequired)
    {
        return EDeviceHealth::Warning;
    }

    switch (ConnectionStatus)
    {
        case ELiveLinkDeviceConnectionStatus::Connected:    return bIdentified ? EDeviceHealth::Good : EDeviceHealth::Info;
        case ELiveLinkDeviceConnectionStatus::Connecting:    return EDeviceHealth::Warning;
        case ELiveLinkDeviceConnectionStatus::Disconnecting: return EDeviceHealth::Warning;
        case ELiveLinkDeviceConnectionStatus::Disconnected:  return EDeviceHealth::Warning;
        default:                                             return EDeviceHealth::Error;
    }
}

FText ULiveLinkOBSDevice::GetHealthText() const
{
    if (bAuthFailed)
    {
        return LOCTEXT("HealthText_AuthFailed", "Authentication failed. Check password.");
    }
    if (bPasswordRequired)
    {
        return LOCTEXT("HealthText_PasswordRequired", "OBS requires a password. Enter one in device settings.");
    }

    switch (ConnectionStatus)
    {
        case ELiveLinkDeviceConnectionStatus::Connected:
            return bIdentified
                ? FText::GetEmpty()
                : LOCTEXT("HealthText_Authenticating", "Authenticating...");
        case ELiveLinkDeviceConnectionStatus::Connecting:
            return LOCTEXT("HealthText_Connecting", "Connecting...");
        case ELiveLinkDeviceConnectionStatus::Disconnecting:
            return LOCTEXT("HealthText_Disconnecting", "Disconnecting...");
        default:
            return LOCTEXT("HealthText_Disconnected", "Not connected");
    }
}

void ULiveLinkOBSDevice::OnDeviceAdded()
{
    Super::OnDeviceAdded();

    if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkRecordingSession::GetModularFeatureName()))
    {
        ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
        Session.OnSlateNameChanged().AddUObject(this, &ULiveLinkOBSDevice::HandleSlateNameChanged);
        Session.OnTakeNumberChanged().AddUObject(this, &ULiveLinkOBSDevice::HandleTakeNumberChanged);
    }
    Connect();
}

void ULiveLinkOBSDevice::OnDeviceRemoved()
{
    if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkRecordingSession::GetModularFeatureName()))
    {
        ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
        Session.OnSlateNameChanged().RemoveAll(this);
        Session.OnTakeNumberChanged().RemoveAll(this);
    }
    Disconnect();

    Super::OnDeviceRemoved();
}

void ULiveLinkOBSDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
    Super::OnSettingChanged(InPropertyChangedEvent);

    const FName Changed = InPropertyChangedEvent.GetPropertyName();

    static const FName HostName     = GET_MEMBER_NAME_CHECKED(ULiveLinkOBSDeviceSettings, Host);
    static const FName PortName     = GET_MEMBER_NAME_CHECKED(ULiveLinkOBSDeviceSettings, Port);
    static const FName PasswordName = GET_MEMBER_NAME_CHECKED(ULiveLinkOBSDeviceSettings, Password);
    static const FName FilenameFormatName = GET_MEMBER_NAME_CHECKED(ULiveLinkOBSDeviceSettings, FilenameFormat);

    if (Changed == HostName || Changed == PortName || Changed == PasswordName)
    {
        bReconnectAfterSettingsChange = true;
        Disconnect();
    }
}


// -----------------------------------------------------------------------
// ILiveLinkDeviceCapability_Connection
// -----------------------------------------------------------------------

ELiveLinkDeviceConnectionStatus ULiveLinkOBSDevice::GetConnectionStatus_Implementation() const
{
    return ConnectionStatus;
}

FString ULiveLinkOBSDevice::GetHardwareId_Implementation() const
{
    const ULiveLinkOBSDeviceSettings* S = GetSettings();
    return FString::Printf(TEXT("%s:%d"), *S->Host, S->Port);
}

bool ULiveLinkOBSDevice::Connect_Implementation()
{
    if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected ||
        ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connecting)
    {
        return false;
    }
    Connect();
    return true;
}

bool ULiveLinkOBSDevice::Disconnect_Implementation()
{
    if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
    {
        return false;
    }
    Disconnect();
    return true;
}

void ULiveLinkOBSDevice::SetConnectionStatus(ELiveLinkDeviceConnectionStatus InStatus)
{
    ConnectionStatus = InStatus;
    ILiveLinkDeviceCapability_Connection::SetConnectionStatus(InStatus);
}


// -----------------------------------------------------------------------
// ILiveLinkDeviceCapability_Recording
// -----------------------------------------------------------------------

bool ULiveLinkOBSDevice::StartRecording_Implementation()
{
    if (bIsRecording || !bIdentified)
    {
        return false;
    }

    // Capture current session slate/take for accurate metadata when recording stops.
    if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkRecordingSession::GetModularFeatureName()))
    {
        ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
        RecordingSlateName = Session.GetSlateName();
        RecordingTakeNumber = Session.GetTakeNumber();
    }

    // --- Recording start sequence (4 requests, all sent synchronously) ---
    //
    // OBS WebSocket processes requests sequentially per connection, so the order is:
    //   1. GetProfileParameter  → OBS reads current FilenameFormatting, sends response
    //   2. SetProfileParameter  → OBS sets Hub filename
    //   3. StartRecord          → OBS starts recording (latches the Hub filename)
    //   4. SetProfileParameter  → OBS restores original (sent from #1's response callback)
    //
    // Request #1's response callback fires after OBS has processed #1 but the callback
    // itself runs on our game thread, by which time we've already sent #2 and #3. The
    // restore (#4) is sent from the callback and arrives at OBS after #2/#3 are queued,
    // so OBS processes them in the correct order: read → set → start → restore.
    //
    // If #1 fails, we fall back to whatever value was previously cached (from connect
    // time or a prior recording). If nothing is cached, we skip the restore entirely —
    // recording still works, filename preservation is best-effort.

    // 1. Query the current FilenameFormatting — response callback sends the restore (#4).
    TSharedRef<FJsonObject> QueryData = MakeShared<FJsonObject>();
    QueryData->SetStringField(TEXT("parameterCategory"), TEXT("Output"));
    QueryData->SetStringField(TEXT("parameterName"),     TEXT("FilenameFormatting"));

    SendRequest(TEXT("GetProfileParameter"), QueryData,
        [WeakThis = TWeakObjectPtr<ULiveLinkOBSDevice>(this)](const TSharedRef<FJsonObject>& ResponseData, bool bSuccess)
        {
            TStrongObjectPtr<ULiveLinkOBSDevice> This = WeakThis.Pin();
            if (!This.IsValid())
            {
                return;
            }

            if (bSuccess)
            {
                FString Value;
                if (ResponseData->TryGetStringField(TEXT("parameterValue"), Value))
                {
                    This->OriginalFilenameFormatting = MoveTemp(Value);
                    This->bOriginalFilenameCached = true;
                }
            }
            else
            {
                UE_LOGFMT(LogLiveLinkOBSDevice, Warning,
                    "{DeviceName}: Failed to refresh OBS FilenameFormatting before recording. "
                    "Proceeding with {State} value.",
                    This->GetSettings()->DisplayName,
                    This->bOriginalFilenameCached ? TEXT("previously cached") : TEXT("no cached"));
            }

            // 4. Restore the original filename. OBS has already started recording with the
            // Hub filename by the time it processes this request.
            This->RestoreOriginalFilenameFormatting();
        });

    // 2. Set the Hub-computed filename.
    const FString HubFilename = EvaluateFilenameFormat();

    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Setting OBS recording filename to '{Filename}'.",
        GetSettings()->DisplayName, HubFilename);

    TSharedRef<FJsonObject> SetFilenameData = MakeShared<FJsonObject>();
    SetFilenameData->SetStringField(TEXT("parameterCategory"), TEXT("Output"));
    SetFilenameData->SetStringField(TEXT("parameterName"),     TEXT("FilenameFormatting"));
    SetFilenameData->SetStringField(TEXT("parameterValue"),    HubFilename);
    SendRequest(TEXT("SetProfileParameter"), SetFilenameData);

    // 3. Start recording.
    if (SendRequest(TEXT("StartRecord")).IsEmpty())
    {
        return false;
    }

    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Sending StartRecord.", GetSettings()->DisplayName);

    return true;
}

bool ULiveLinkOBSDevice::StopRecording_Implementation()
{
    if (!bIdentified)
    {
        return false;
    }

    // Intentionally not guarding on bIsRecording. The recording state is updated
    // asynchronously via the RecordStateChanged event, so a rapid start/stop sequence
    // can reach here before OBS has confirmed the start. Always send StopRecord and
    // let OBS reject it if nothing is recording.

    if (SendRequest(TEXT("StopRecord")).IsEmpty())
    {
        return false;
    }

    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Sending StopRecord.", GetSettings()->DisplayName);

    return true;
}

bool ULiveLinkOBSDevice::IsRecording_Implementation() const
{
    return bIsRecording;
}


// -----------------------------------------------------------------------
// UBaseIngestLiveLinkDevice interface (Ingest)
// -----------------------------------------------------------------------

FString ULiveLinkOBSDevice::GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const
{
    FScopeLock Lock(&FullTakePathsMutex);

    if (const FString* TakePath = FullTakePaths.Find(InTakeId))
    {
        return *TakePath;
    }

    return FString();
}

void ULiveLinkOBSDevice::RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions)
{
    auto Task = [WeakThis = TWeakObjectPtr<ULiveLinkOBSDevice>(this),
        ProcessHandle = TStrongObjectPtr<const UIngestCapability_ProcessHandle>(InProcessHandle),
        IngestOptions = TStrongObjectPtr<const UIngestCapability_Options>(InIngestOptions)]()
        {
            TStrongObjectPtr<ULiveLinkOBSDevice> This = WeakThis.Pin();

            if (!This.IsValid())
            {
                UE_LOG(LogLiveLinkOBSDevice, Warning, TEXT("Failed to ingest take, the device has been destroyed"));
                return;
            }

            static constexpr uint32 NumberOfTasks = 2; // Convert, Upload

            using namespace UE::CaptureManager;
            TSharedPtr<FTaskProgress> TaskProgress
                = MakeShared<FTaskProgress>(NumberOfTasks, FTaskProgress::FProgressReporter::CreateLambda([WeakThis, ProcessHandle](double InProgress)
                    {
                        TStrongObjectPtr<ULiveLinkOBSDevice> PinnedThis = WeakThis.Pin();

                        if (!PinnedThis.IsValid())
                        {
                            return;
                        }

                        PinnedThis->ExecuteProcessProgressReporter(ProcessHandle.Get(), InProgress);
                    }));

            This->Super::IngestTake(ProcessHandle.Get(), IngestOptions.Get(), MoveTemp(TaskProgress));
        };

    UE::Tasks::Launch(UE_SOURCE_LOCATION, MoveTemp(Task), UE::Tasks::ETaskPriority::BackgroundNormal);
}

void ULiveLinkOBSDevice::RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback)
{
    RemoveAllTakes();

    {
        FScopeLock Lock(&FullTakePathsMutex);
        FullTakePaths.Empty();
    }

    if (CachedRecordDirectory.IsEmpty())
    {
        UE_LOGFMT(LogLiveLinkOBSDevice, Warning, "{DeviceName}: No OBS record directory cached. Connect to OBS first.",
            GetSettings()->DisplayName);
        ExecuteUpdateTakeListCallback(InCallback, TArray<int32>());
        return;
    }

    if (!FPaths::DirectoryExists(CachedRecordDirectory))
    {
        UE_LOGFMT(LogLiveLinkOBSDevice, Warning, "{DeviceName}: OBS record directory does not exist: {Dir}",
            GetSettings()->DisplayName, CachedRecordDirectory);
        ExecuteUpdateTakeListCallback(InCallback, TArray<int32>());
        return;
    }

    TArray<FString> VideoFiles;
    IFileManager::Get().IterateDirectoryRecursively(*CachedRecordDirectory, [&](const TCHAR* InFileNameOrDirectory, bool bInIsDirectory)
        {
            if (IsUpdateTakeListAbortRequested())
            {
                return false;
            }

            if (!bInIsDirectory && IsOBSVideoFile(InFileNameOrDirectory))
            {
                VideoFiles.Add(InFileNameOrDirectory);
            }

            return true;
        });

    // Derive a discovery expression from the current FilenameFormat for primary parsing.
    const FString DerivedExpression = DeriveDiscoveryExpression(GetSettings()->FilenameFormat);
    static const FString FallbackExpression = TEXT("<Auto>");

    for (const FString& VideoFile : VideoFiles)
    {
        if (IsUpdateTakeListAbortRequested())
        {
            ExecuteUpdateTakeListCallback(InCallback, TArray<int32>());
            return;
        }

        // Two-tier parsing: try derived expression from current FilenameFormat first,
        // then fall back to <Auto> (full filename as slate) for files that don't match.
        TOptional<FTakeMetadata> TakeInfoResult = ReadTakeFromFile(VideoFile, DerivedExpression);

        if (!TakeInfoResult.IsSet() && FallbackExpression != DerivedExpression)
        {
            TakeInfoResult = ReadTakeFromFile(VideoFile, FallbackExpression);
        }

        if (TakeInfoResult.IsSet())
        {
            int32 TakeId = AddTake(MoveTemp(TakeInfoResult.GetValue()));

            {
                FScopeLock Lock(&FullTakePathsMutex);
                FullTakePaths.Add(TakeId, VideoFile);
            }

            PublishEvent<FIngestCapability_TakeAddedEvent>(TakeId);
        }
    }

    ExecuteUpdateTakeListCallback(InCallback, Execute_GetTakeIdentifiers(this));
}


// -----------------------------------------------------------------------
// WebSocket lifecycle
// -----------------------------------------------------------------------

void ULiveLinkOBSDevice::Connect()
{
    if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Disconnected)
    {
        return;
    }

    bAuthFailed = false;
    bPasswordRequired = false;

    const ULiveLinkOBSDeviceSettings* S = GetSettings();
    const FString Url = FString::Printf(TEXT("ws://%s:%d"), *S->Host, S->Port);

    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Connecting to {Url}.", S->DisplayName, Url);

    WebSocket = FWebSocketsModule::Get().CreateWebSocket(Url, TEXT("obswebsocket.v5"));
    WebSocket->OnConnected()      .AddUObject(this, &ULiveLinkOBSDevice::HandleConnected);
    WebSocket->OnMessage()        .AddUObject(this, &ULiveLinkOBSDevice::HandleMessage);
    WebSocket->OnClosed()         .AddUObject(this, &ULiveLinkOBSDevice::HandleClosed);
    WebSocket->OnConnectionError().AddUObject(this, &ULiveLinkOBSDevice::HandleError);

    SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connecting);
    WebSocket->Connect();
}

void ULiveLinkOBSDevice::Disconnect()
{
    bIntentionalDisconnect = true;

    if (WebSocket && WebSocket->IsConnected())
    {
        SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnecting);
        WebSocket->Close();
    }
    else
    {
        OnDisconnect();
    }
}

void ULiveLinkOBSDevice::OnDisconnect()
{
    bIdentified    = false;
    bIsRecording   = false;
    RequestCounter = 0;
    PendingCallbacks.Empty();
    CachedRecordDirectory.Empty();
    OriginalFilenameFormatting.Empty();
    bOriginalFilenameCached = false;

    if (WebSocket)
    {
        WebSocket->OnConnected()      .RemoveAll(this);
        WebSocket->OnMessage()        .RemoveAll(this);
        WebSocket->OnClosed()         .RemoveAll(this);
        WebSocket->OnConnectionError().RemoveAll(this);
        WebSocket.Reset();
    }

    SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);

    // Cancel any previously scheduled reconnect.
    if (ReconnectTickerHandle.IsValid())
    {
        FTSTicker::GetCoreTicker().RemoveTicker(ReconnectTickerHandle);
        ReconnectTickerHandle.Reset();
    }

    if (!bIntentionalDisconnect)
    {
        UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Unexpected disconnect, reconnecting in {Delay}s.",
            GetSettings()->DisplayName, ReconnectDelay);

        ReconnectTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
            FTickerDelegate::CreateWeakLambda(this, [this](float) -> bool
            {
                ReconnectTickerHandle.Reset();
                Connect();
                return false; // one-shot
            }),
            ReconnectDelay
        );
    }

    bIntentionalDisconnect = false;

    if (bReconnectAfterSettingsChange)
    {
        bReconnectAfterSettingsChange = false;
        Connect();
    }
}


// -----------------------------------------------------------------------
// WebSocket event handlers
// -----------------------------------------------------------------------

void ULiveLinkOBSDevice::HandleConnected()
{
    // OBS sends Hello (op 0) immediately after connection; wait for it before doing anything.
    // Stay in Connecting state until fully identified and the record directory is cached.
    // This ensures Capture Manager's auto-refresh fires only when we're ready to serve takes.
    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: WebSocket connected; waiting for Hello.", GetSettings()->DisplayName);
}

void ULiveLinkOBSDevice::HandleMessage(const FString& InMessage)
{
    TSharedPtr<FJsonObject> Root;
    TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InMessage);
    if (!FJsonSerializer::Deserialize(Reader, Root) || !Root.IsValid())
    {
        UE_LOGFMT(LogLiveLinkOBSDevice, Warning, "{DeviceName}: Failed to parse JSON: {Msg}",
            GetSettings()->DisplayName, InMessage);
        return;
    }

    int32 OpCode = 0;
    if (!Root->TryGetNumberField(TEXT("op"), OpCode))
    {
        UE_LOGFMT(LogLiveLinkOBSDevice, Warning, "{DeviceName}: Message missing 'op' field.", GetSettings()->DisplayName);
        return;
    }

    const TSharedPtr<FJsonObject>* DataObj = nullptr;
    Root->TryGetObjectField(TEXT("d"), DataObj);
    TSharedRef<FJsonObject> Data = DataObj ? DataObj->ToSharedRef() : MakeShared<FJsonObject>();

    switch (OpCode)
    {
        case OBSWebSocketOpCode::Hello:
            HandleHello(Data);
            break;

        case OBSWebSocketOpCode::Identified:
            UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Identified. Device ready.", GetSettings()->DisplayName);
            bIdentified = true;
            QueryOriginalFilenameFormatting();
            QueryRecordDirectory();
            // SetConnectionStatus(Connected) is deferred to QueryRecordDirectory's response
            // callback to ensure the take list is not refreshed before the record directory is cached.
            break;

        case OBSWebSocketOpCode::Event:
            HandleEvent(Data);
            break;

        case OBSWebSocketOpCode::Response:
            HandleResponse(Data);
            break;

        default:
            break;
    }
}

void ULiveLinkOBSDevice::HandleClosed(int32 StatusCode, const FString& Reason, bool bWasClean)
{
    UE_LOGFMT(LogLiveLinkOBSDevice, Log,
        "{DeviceName}: WebSocket closed (code={StatusCode}, reason='{Reason}', clean={WasClean}).",
        GetSettings()->DisplayName, StatusCode, Reason, bWasClean);

    // OBS close code 4009 means the server rejected our authentication hash.
    // Retrying with the same password would just loop forever, so treat this as
    // an intentional disconnect and surface a clear error instead.
    static constexpr int32 OBSCloseCode_AuthenticationFailed = 4009;
    if (StatusCode == OBSCloseCode_AuthenticationFailed)
    {
        bIntentionalDisconnect = true; // always suppress reconnect on auth failure

        if (bPasswordRequired)
        {
            // Blank password; bPasswordRequired was already set in HandleHello.
            // Leave it set so the Warning state persists; don't escalate to Error.
            UE_LOGFMT(LogLiveLinkOBSDevice, Warning,
                "{DeviceName}: OBS requires authentication, no password configured.",
                GetSettings()->DisplayName);
        }
        else
        {
            // Password was provided but OBS rejected it.
            UE_LOGFMT(LogLiveLinkOBSDevice, Error,
                "{DeviceName}: Authentication failed. Verify the password in device settings.",
                GetSettings()->DisplayName);
            bAuthFailed = true;
        }
    }

    OnDisconnect();
}

void ULiveLinkOBSDevice::HandleError(const FString& Error)
{
    UE_LOGFMT(LogLiveLinkOBSDevice, Error, "{DeviceName}: WebSocket error: {Error}",
        GetSettings()->DisplayName, Error);
    OnDisconnect();
}


// -----------------------------------------------------------------------
// OBS op handlers
// -----------------------------------------------------------------------

void ULiveLinkOBSDevice::HandleHello(const TSharedRef<FJsonObject>& InPayload)
{
    const TSharedPtr<FJsonObject>* AuthObj = nullptr;
    if (InPayload->TryGetObjectField(TEXT("authentication"), AuthObj))
    {
        FString Challenge, Salt;
        (*AuthObj)->TryGetStringField(TEXT("challenge"), Challenge);
        (*AuthObj)->TryGetStringField(TEXT("salt"), Salt);

        if (GetSettings()->Password.IsEmpty())
        {
            UE_LOGFMT(LogLiveLinkOBSDevice, Warning,
                "{DeviceName}: OBS requires authentication but no password is configured.",
                GetSettings()->DisplayName);
            bPasswordRequired = true;
        }

        SendIdentify(ComputeAuthResponse(GetSettings()->Password, Challenge, Salt));
    }
    else
    {
        SendIdentify(FString());
    }
}

void ULiveLinkOBSDevice::HandleEvent(const TSharedRef<FJsonObject>& InPayload)
{
    FString EventType;
    if (!InPayload->TryGetStringField(TEXT("eventType"), EventType))
    {
        return;
    }

    if (EventType == TEXT("RecordStateChanged"))
    {
        const TSharedPtr<FJsonObject>* EventData = nullptr;
        InPayload->TryGetObjectField(TEXT("eventData"), EventData);
        if (!EventData)
        {
            return;
        }

        FString OutputState;
        (*EventData)->TryGetStringField(TEXT("outputState"), OutputState);

        const bool bNowRecording = (OutputState == TEXT("OBS_WEBSOCKET_OUTPUT_STARTED"));
        const bool bStopped = (OutputState == TEXT("OBS_WEBSOCKET_OUTPUT_STOPPED"));

        if (bIsRecording != bNowRecording)
        {
            bIsRecording = bNowRecording;
            UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Recording state -> {State}.",
                GetSettings()->DisplayName, bIsRecording ? TEXT("Recording") : TEXT("Stopped"));
        }

        // When recording stops, OBS includes the output file path. Auto-add it as a take.
        if (bStopped)
        {
            FString OutputPath;
            if ((*EventData)->TryGetStringField(TEXT("outputPath"), OutputPath) && !OutputPath.IsEmpty())
            {
                AddTakeFromRecording(OutputPath);
            }
        }
    }
}

void ULiveLinkOBSDevice::HandleResponse(const TSharedRef<FJsonObject>& InPayload)
{
    // Route response to pending callback if one exists.
    FString RequestId;
    InPayload->TryGetStringField(TEXT("requestId"), RequestId);

    // Check request status.
    bool bSuccess = false;
    const TSharedPtr<FJsonObject>* StatusObj = nullptr;
    if (InPayload->TryGetObjectField(TEXT("requestStatus"), StatusObj))
    {
        (*StatusObj)->TryGetBoolField(TEXT("result"), bSuccess);

        if (!bSuccess)
        {
            FString RequestType;
            int32 StatusCode = 0;
            InPayload->TryGetStringField(TEXT("requestType"), RequestType);
            (*StatusObj)->TryGetNumberField(TEXT("code"), StatusCode);
            UE_LOGFMT(LogLiveLinkOBSDevice, Warning,
                "{DeviceName}: Request '{RequestType}' failed with OBS status code {StatusCode}.",
                GetSettings()->DisplayName, RequestType, StatusCode);
        }
    }

    // Extract response data (may be absent for void responses).
    const TSharedPtr<FJsonObject>* ResponseDataObj = nullptr;
    InPayload->TryGetObjectField(TEXT("responseData"), ResponseDataObj);
    TSharedRef<FJsonObject> ResponseData = ResponseDataObj ? ResponseDataObj->ToSharedRef() : MakeShared<FJsonObject>();

    // Fire callback if registered.
    FResponseCallback Callback;
    if (!RequestId.IsEmpty() && PendingCallbacks.RemoveAndCopyValue(RequestId, Callback))
    {
        Callback(ResponseData, bSuccess);
    }
}


// -----------------------------------------------------------------------
// Request helpers
// -----------------------------------------------------------------------

void ULiveLinkOBSDevice::SendIdentify(const FString& InAuthResponse)
{
    TSharedRef<FJsonObject> Data = MakeShared<FJsonObject>();
    Data->SetNumberField(TEXT("rpcVersion"), 1);
    if (!InAuthResponse.IsEmpty())
    {
        Data->SetStringField(TEXT("authentication"), InAuthResponse);
    }
    Data->SetNumberField(TEXT("eventSubscriptions"), 1 << 6); // Outputs category; includes RecordStateChanged

    TSharedRef<FJsonObject> Root = MakeShared<FJsonObject>();
    Root->SetNumberField(TEXT("op"), OBSWebSocketOpCode::Identify);
    Root->SetObjectField(TEXT("d"), Data);

    FString Out;
    TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Out);
    FJsonSerializer::Serialize(Root, Writer);
    SendRaw(Out);
}

FString ULiveLinkOBSDevice::SendRequest(const FString& InRequestType, TSharedPtr<FJsonObject> InRequestData, FResponseCallback InCallback)
{
    if (!WebSocket || !WebSocket->IsConnected())
    {
        return FString();
    }

    const FString RequestId = FString::Printf(TEXT("llhub-%u"), ++RequestCounter);

    TSharedRef<FJsonObject> Data = MakeShared<FJsonObject>();
    Data->SetStringField(TEXT("requestType"), InRequestType);
    Data->SetStringField(TEXT("requestId"),   RequestId);
    if (InRequestData.IsValid())
    {
        Data->SetObjectField(TEXT("requestData"), InRequestData.ToSharedRef());
    }

    TSharedRef<FJsonObject> Root = MakeShared<FJsonObject>();
    Root->SetNumberField(TEXT("op"), OBSWebSocketOpCode::Request);
    Root->SetObjectField(TEXT("d"), Data);

    FString Out;
    TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Out);
    FJsonSerializer::Serialize(Root, Writer);
    SendRaw(Out);

    if (InCallback)
    {
        PendingCallbacks.Add(RequestId, MoveTemp(InCallback));
    }

    return RequestId;
}

void ULiveLinkOBSDevice::SendRaw(const FString& InJson)
{
    if (WebSocket && WebSocket->IsConnected())
    {
        WebSocket->Send(InJson);
    }
}


// -----------------------------------------------------------------------
// Hub session callbacks
// -----------------------------------------------------------------------

void ULiveLinkOBSDevice::HandleSlateNameChanged(FStringView InSlateName)
{
    // Slate name changes are captured at StartRecording time; no need to push eagerly to OBS.
    (void)InSlateName;
}

void ULiveLinkOBSDevice::HandleTakeNumberChanged(int32 InTakeNumber)
{
    // Take number changes are captured at StartRecording time; no need to push eagerly to OBS.
    (void)InTakeNumber;
}

FString ULiveLinkOBSDevice::EvaluateFilenameFormat() const
{
    UNamingTokensEngineSubsystem* TokenSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();

    FNamingTokenFilterArgs FilterArgs;
    FilterArgs.AdditionalNamespacesToInclude.Add(TEXT("llh"));

    FNamingTokenResultData Result = TokenSubsystem->EvaluateTokenText(
        FText::FromString(GetSettings()->FilenameFormat),
        FilterArgs
    );

    return Result.EvaluatedText.ToString();
}

void ULiveLinkOBSDevice::QueryOriginalFilenameFormatting()
{
    TSharedRef<FJsonObject> RequestData = MakeShared<FJsonObject>();
    RequestData->SetStringField(TEXT("parameterCategory"), TEXT("Output"));
    RequestData->SetStringField(TEXT("parameterName"),     TEXT("FilenameFormatting"));

    SendRequest(TEXT("GetProfileParameter"), RequestData,
        [WeakThis = TWeakObjectPtr<ULiveLinkOBSDevice>(this)](const TSharedRef<FJsonObject>& ResponseData, bool bSuccess)
        {
            TStrongObjectPtr<ULiveLinkOBSDevice> This = WeakThis.Pin();
            if (!This.IsValid())
            {
                return;
            }

            if (bSuccess)
            {
                FString Value;
                if (ResponseData->TryGetStringField(TEXT("parameterValue"), Value))
                {
                    This->OriginalFilenameFormatting = MoveTemp(Value);
                    This->bOriginalFilenameCached = true;

                    UE_LOGFMT(LogLiveLinkOBSDevice, Log,
                        "{DeviceName}: Cached original OBS FilenameFormatting: '{Value}'.",
                        This->GetSettings()->DisplayName, This->OriginalFilenameFormatting);
                }
            }
            else
            {
                UE_LOGFMT(LogLiveLinkOBSDevice, Warning,
                    "{DeviceName}: Failed to query original OBS FilenameFormatting. "
                    "The user's OBS filename format may be overwritten during Hub recordings.",
                    This->GetSettings()->DisplayName);
            }
        });
}

void ULiveLinkOBSDevice::RestoreOriginalFilenameFormatting()
{
    if (!bOriginalFilenameCached)
    {
        UE_LOGFMT(LogLiveLinkOBSDevice, Warning,
            "{DeviceName}: Cannot restore OBS filename — original value was not cached.",
            GetSettings()->DisplayName);
        return;
    }

    UE_LOGFMT(LogLiveLinkOBSDevice, Log,
        "{DeviceName}: Restoring original OBS FilenameFormatting: '{Value}'.",
        GetSettings()->DisplayName, OriginalFilenameFormatting);

    TSharedRef<FJsonObject> RequestData = MakeShared<FJsonObject>();
    RequestData->SetStringField(TEXT("parameterCategory"), TEXT("Output"));
    RequestData->SetStringField(TEXT("parameterName"),     TEXT("FilenameFormatting"));
    RequestData->SetStringField(TEXT("parameterValue"),    OriginalFilenameFormatting);

    SendRequest(TEXT("SetProfileParameter"), RequestData);
}


// -----------------------------------------------------------------------
// Ingest helpers
// -----------------------------------------------------------------------

void ULiveLinkOBSDevice::QueryRecordDirectory()
{
    SendRequest(TEXT("GetRecordDirectory"), nullptr,
        [WeakThis = TWeakObjectPtr<ULiveLinkOBSDevice>(this)](const TSharedRef<FJsonObject>& ResponseData, bool bSuccess)
        {
            TStrongObjectPtr<ULiveLinkOBSDevice> This = WeakThis.Pin();
            if (!This.IsValid())
            {
                return;
            }

            if (bSuccess)
            {
                FString Directory;
                if (ResponseData->TryGetStringField(TEXT("recordDirectory"), Directory))
                {
                    This->CachedRecordDirectory = MoveTemp(Directory);
                    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: OBS record directory: {Dir}",
                        This->GetSettings()->DisplayName, This->CachedRecordDirectory);
                }
            }

            // Now that we have the record directory (or failed trying), transition to Connected.
            // This triggers Capture Manager's auto-refresh of the take list.
            This->SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
        });
}

void ULiveLinkOBSDevice::AddTakeFromRecording(const FString& InOutputPath)
{
    UE_LOGFMT(LogLiveLinkOBSDevice, Log, "{DeviceName}: Recording completed: {Path}",
        GetSettings()->DisplayName, InOutputPath);

    FTakeMetadata TakeMetadata;
    TakeMetadata.Version.Major = 4;
    TakeMetadata.Version.Minor = 1;
    TakeMetadata.UniqueId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
    TakeMetadata.DateTime = FDateTime::Now();
    TakeMetadata.Device.Model = TEXT("OBSStudio");

    // Use the slate/take captured at StartRecording time for accurate metadata.
    if (!RecordingSlateName.IsEmpty())
    {
        TakeMetadata.Slate = RecordingSlateName;
        TakeMetadata.TakeNumber = (RecordingTakeNumber != INDEX_NONE) ? RecordingTakeNumber : 1;
    }
    else
    {
        // Fallback: use filename as slate.
        TakeMetadata.Slate = FPaths::GetBaseFilename(InOutputPath);
        TakeMetadata.TakeNumber = 1;
    }

    FTakeMetadata::FVideo Video;
    Video.Name = TEXT("video");
    Video.Path = InOutputPath;
    Video.PathType = FTakeMetadata::FVideo::EPathType::File;
    Video.Format = FPaths::GetExtension(InOutputPath);
    Video.Orientation = FTakeMetadata::FVideo::EOrientation::Original;

    TakeMetadata.Video.Add(MoveTemp(Video));

    // Extract a thumbnail frame from the recorded video.
    using namespace UE::CaptureManager;
    FVideoDeviceThumbnailExtractor ThumbnailExtractor;
    TOptional<FTakeThumbnailData::FRawImage> RawImageOpt = ThumbnailExtractor.ExtractThumbnail(InOutputPath);
    if (RawImageOpt.IsSet())
    {
        TakeMetadata.Thumbnail = MoveTemp(RawImageOpt.GetValue());
    }

    int32 TakeId = AddTake(MoveTemp(TakeMetadata));

    {
        FScopeLock Lock(&FullTakePathsMutex);
        FullTakePaths.Add(TakeId, InOutputPath);
    }

    PublishEvent<FIngestCapability_TakeAddedEvent>(TakeId);

    // Clear the captured recording session state.
    RecordingSlateName.Empty();
    RecordingTakeNumber = INDEX_NONE;
}

TOptional<FTakeMetadata> ULiveLinkOBSDevice::ReadTakeFromFile(const FString& InVideoFilePath, const FString& InDiscoveryExpression) const
{
    FTakeMetadata TakeMetadata;

    FFileStatData FileData = IFileManager::Get().GetStatData(*InVideoFilePath);

    FString FileName = FPaths::GetBaseFilename(InVideoFilePath);

    FString SlateName;
    FString Name;
    int32 TakeNumber = INDEX_NONE;

    if (InDiscoveryExpression != TEXT("<Auto>"))
    {
        FTakeDiscoveryExpressionParser TokenParser(InDiscoveryExpression, FileName, OBSSupportedVideoFormats::Delimiters);
        bool bParse = TokenParser.Parse();

        if (!bParse)
        {
            return {};
        }

        SlateName = TokenParser.GetSlateName();
        TakeNumber = TokenParser.GetTakeNumber();
        Name = TokenParser.GetName();
    }

    if (SlateName.IsEmpty())
    {
        SlateName = MoveTemp(FileName);
    }

    if (TakeNumber == INDEX_NONE)
    {
        TakeNumber = 1;
    }

    if (Name.IsEmpty())
    {
        Name = TEXT("video");
    }

    TakeMetadata.Version.Major = 4;
    TakeMetadata.Version.Minor = 1;

    TakeMetadata.Slate = MoveTemp(SlateName);
    TakeMetadata.TakeNumber = TakeNumber;
    TakeMetadata.UniqueId = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower);
    TakeMetadata.DateTime = FileData.bIsValid ? FileData.CreationTime : FDateTime::Now();
    TakeMetadata.Device.Model = TEXT("OBSStudio");

    FTakeMetadata::FVideo Video;
    Video.Name = MoveTemp(Name);
    Video.Format = FPaths::GetExtension(InVideoFilePath);
    Video.Path = InVideoFilePath;
    Video.PathType = FTakeMetadata::FVideo::EPathType::File;
    Video.Orientation = FTakeMetadata::FVideo::EOrientation::Original;

    TakeMetadata.Video.Add(MoveTemp(Video));

    // Extract a thumbnail frame from the video file.
    using namespace UE::CaptureManager;
    FVideoDeviceThumbnailExtractor ThumbnailExtractor;
    TOptional<FTakeThumbnailData::FRawImage> RawImageOpt = ThumbnailExtractor.ExtractThumbnail(InVideoFilePath);
    if (RawImageOpt.IsSet())
    {
        TakeMetadata.Thumbnail = MoveTemp(RawImageOpt.GetValue());
    }

    return TakeMetadata;
}

bool ULiveLinkOBSDevice::IsOBSVideoFile(const FString& InFilePath)
{
    FString Extension = FPaths::GetExtension(InFilePath);
    return OBSSupportedVideoFormats::Extensions.Contains(Extension);
}

FString ULiveLinkOBSDevice::DeriveDiscoveryExpression(const FString& InFilenameFormat)
{
    // Convert NamingTokens syntax to TakeDiscoveryExpression syntax.
    // {slate} -> <Slate>, {take} -> <Take>, any other {token} -> <Any>, literals pass through.

    FString Result;
    Result.Reserve(InFilenameFormat.Len());

    int32 Index = 0;
    while (Index < InFilenameFormat.Len())
    {
        if (InFilenameFormat[Index] == '{')
        {
            int32 CloseIndex = InFilenameFormat.Find(TEXT("}"), ESearchCase::CaseSensitive, ESearchDir::FromStart, Index + 1);
            if (CloseIndex != INDEX_NONE)
            {
                FString TokenName = InFilenameFormat.Mid(Index + 1, CloseIndex - Index - 1).ToLower();

                if (TokenName == TEXT("slate"))
                {
                    Result += TEXT("<Slate>");
                }
                else if (TokenName == TEXT("take"))
                {
                    Result += TEXT("<Take>");
                }
                else
                {
                    Result += TEXT("<Any>");
                }

                Index = CloseIndex + 1;
            }
            else
            {
                // Malformed token — pass through literally.
                Result += InFilenameFormat[Index];
                ++Index;
            }
        }
        else
        {
            Result += InFilenameFormat[Index];
            ++Index;
        }
    }

    return Result;
}


// -----------------------------------------------------------------------
// Auth
// -----------------------------------------------------------------------

FString ULiveLinkOBSDevice::ComputeAuthResponse(const FString& InPassword, const FString& InChallenge, const FString& InSalt)
{
    // OBS WebSocket v5 auth:
    //   base64( SHA256( base64( SHA256( password + salt ) ) + challenge ) )

    auto SHA256AndBase64 = [](const FString& InStr) -> FString
    {
        const FTCHARToUTF8 Utf8(*InStr);
        uint8 Digest[32];
#if LLOBS_WITH_OPENSSL
        SHA256_CTX Ctx;
        SHA256_Init(&Ctx);
        SHA256_Update(&Ctx, Utf8.Get(), Utf8.Length());
        SHA256_Final(Digest, &Ctx);
#else
        FMemory::Memzero(Digest);
#endif
        return FBase64::Encode(Digest, sizeof(Digest));
    };

    return SHA256AndBase64(SHA256AndBase64(InPassword + InSalt) + InChallenge);
}


#undef LOCTEXT_NAMESPACE
