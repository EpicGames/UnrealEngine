// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "BaseIngestLiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "LiveLinkOBSDevice.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkOBSDevice, Log, All);

class IWebSocket;

// -----------------------------------------------------------------------
// Settings
// -----------------------------------------------------------------------

/** Persisted settings for an OBS device instance. */
UCLASS()
class ULiveLinkOBSDeviceSettings : public ULiveLinkDeviceSettings
{
    GENERATED_BODY()

public:

	ULiveLinkOBSDeviceSettings()
	{
		DisplayName = TEXT("OBS Studio");
	}

    /** Host running OBS WebSocket server (e.g. 127.0.0.1 or a remote IP). */
    UPROPERTY(EditAnywhere, Category="OBS Device")
    FString Host = TEXT("127.0.0.1");

    /** Port configured in OBS Tools > WebSocket Server Settings (default 4455). */
    UPROPERTY(EditAnywhere, Category="OBS Device")
    int32 Port = 4455;

    /** Password set in OBS WebSocket Server Settings. Leave empty if authentication is disabled. */
    UPROPERTY(EditAnywhere, Category="OBS Device", meta=(PasswordField=true))
    FString Password;

    /**
     * Format string for the OBS recording filename.
     * Uses NamingTokens syntax; wrap token names in curly braces.
     *
     * LLH tokens (namespace "llh"): {slate}, {take}, {session}, {config}
     * Global tokens: {project}, {user}, {yyyy}, {mm}, {Ddd}, etc.
     *
     * This format is also used to identify and parse takes when browsing the OBS recording
     * directory. For example, {slate}-{take} will correctly extract the slate name and take
     * number from "MySlate-01.mp4". Recordings that don't match the current format are still
     * discovered, using the full filename as the slate name.
     */
    UPROPERTY(EditAnywhere, Category="OBS Device", meta=(NamingTokens))
    FString FilenameFormat = TEXT("{session}/{slate}_tk{take}");
};


// -----------------------------------------------------------------------
// Device
// -----------------------------------------------------------------------

/**
 * Live Link Hub device that controls OBS Studio recording via the OBS WebSocket v5 protocol.
 *
 * Supports:
 *   - Start / Stop recording (driven by the Live Link Hub recording session)
 *   - Setting the recording filename (driven by the Hub slate name)
 *   - Ingest of recorded video files as mono takes into Capture Manager
 */
UCLASS(BlueprintType, meta=(DisplayName="OBS Studio"))
class ULiveLinkOBSDevice : public UBaseIngestLiveLinkDevice
    , public ILiveLinkDeviceCapability_Connection
    , public ILiveLinkDeviceCapability_Recording
{
    GENERATED_BODY()

public:
    ULiveLinkOBSDeviceSettings* GetSettings()             { return GetDeviceSettings<ULiveLinkOBSDeviceSettings>(); }
    const ULiveLinkOBSDeviceSettings* GetSettings() const { return GetDeviceSettings<ULiveLinkOBSDeviceSettings>(); }

    //~ Begin ULiveLinkDevice interface
    virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override { return ULiveLinkOBSDeviceSettings::StaticClass(); }
    virtual EDeviceHealth GetDeviceHealth() const override;
    virtual FText         GetHealthText()   const override;
    virtual void OnDeviceAdded()   override;
    virtual void OnDeviceRemoved() override;
    virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
    //~ End ULiveLinkDevice interface

    //~ Begin ILiveLinkDeviceCapability_Connection interface
    virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
    virtual FString GetHardwareId_Implementation()                                    const override;
    virtual bool    SetHardwareId_Implementation(const FString& InHardwareID)               override { return false; }
    virtual bool    Connect_Implementation()                                                override;
    virtual bool    Disconnect_Implementation()                                             override;
protected:
    virtual void    SetConnectionStatus(ELiveLinkDeviceConnectionStatus InStatus)           override;
    //~ End ILiveLinkDeviceCapability_Connection interface

public:
    //~ Begin ILiveLinkDeviceCapability_Recording interface
    virtual bool StartRecording_Implementation() override;
    virtual bool StopRecording_Implementation()  override;
    virtual bool IsRecording_Implementation() const override;
    //~ End ILiveLinkDeviceCapability_Recording interface

private:
    //~ Begin UBaseIngestLiveLinkDevice interface
    virtual FString GetFullTakePath(UE::CaptureManager::FTakeId InTakeId) const override;
    virtual void RunConvertAndUploadTake(const UIngestCapability_ProcessHandle* InProcessHandle, const UIngestCapability_Options* InIngestOptions) override;
    virtual void RunUpdateTakeList(UIngestCapability_UpdateTakeListCallback* InCallback) override;
    //~ End UBaseIngestLiveLinkDevice interface

    // ----------------------------------------------------------------
    // WebSocket lifecycle
    // ----------------------------------------------------------------

    /** Open the WebSocket connection to OBS. */
    void Connect();

    /** Close the WebSocket connection gracefully. */
    void Disconnect();

    /** Reset transient state (called on any disconnect path). */
    void OnDisconnect();

    // ----------------------------------------------------------------
    // OBS WebSocket v5 protocol
    // ----------------------------------------------------------------

    /** Called when the WebSocket connects. OBS sends a Hello immediately after. */
    void HandleConnected();

    /** Called when the WebSocket receives a text message from OBS. */
    void HandleMessage(const FString& InMessage);

    /** Called when the WebSocket connection is lost. */
    void HandleClosed(int32 StatusCode, const FString& Reason, bool bWasClean);

    /** Called when the WebSocket encounters an error. */
    void HandleError(const FString& Error);

    // ----------------------------------------------------------------
    // OBS WebSocket op handlers
    // ----------------------------------------------------------------

    /** Op 0 (Hello): OBS announces its version and optional auth challenge. */
    void HandleHello(const TSharedRef<FJsonObject>& InPayload);

    /** Op 5 (Event): OBS pushed an event (e.g. RecordStateChanged). */
    void HandleEvent(const TSharedRef<FJsonObject>& InPayload);

    /** Op 7 (Response): OBS is responding to a request we sent. */
    void HandleResponse(const TSharedRef<FJsonObject>& InPayload);

    // ----------------------------------------------------------------
    // OBS WebSocket request helpers
    // ----------------------------------------------------------------

    /** Send an Identify (op 1) message. Pass the computed auth response, or empty string if auth is not required. */
    void SendIdentify(const FString& InAuthResponse);

    /** Callback type for handling async OBS request responses. */
    using FResponseCallback = TFunction<void(const TSharedRef<FJsonObject>& ResponseData, bool bSuccess)>;

    /**
     * Send a request (op 6) to OBS. Returns the request ID used.
     * If a callback is provided, it will be invoked when the response arrives.
     */
    FString SendRequest(const FString& InRequestType, TSharedPtr<FJsonObject> InRequestData = nullptr, FResponseCallback InCallback = nullptr);

    /** Build and send a raw JSON string over the WebSocket. */
    void SendRaw(const FString& InJson);

    // ----------------------------------------------------------------
    // Live Link Hub session callbacks
    // ----------------------------------------------------------------
    void HandleSlateNameChanged(FStringView InSlateName);
    void HandleTakeNumberChanged(int32 InTakeNumber);

    /** Evaluate the FilenameFormat naming-tokens template and return the result. */
    FString EvaluateFilenameFormat() const;

    /**
     * Query the original FilenameFormatting value from OBS and cache it.
     * Called after identification. Also sent as part of the recording start
     * sequence so the cached value is always fresh when we restore.
     */
    void QueryOriginalFilenameFormatting();

    /** Restore the original OBS FilenameFormatting that was cached. */
    void RestoreOriginalFilenameFormatting();

    // ----------------------------------------------------------------
    // Ingest helpers
    // ----------------------------------------------------------------

    /** Query OBS for its configured record directory and cache the result. */
    void QueryRecordDirectory();

    /** Build a take from a completed recording (real-time path, known slate/take). */
    void AddTakeFromRecording(const FString& InOutputPath);

    /** Build a take from a discovered video file (directory scan path, parsed naming). */
    TOptional<FTakeMetadata> ReadTakeFromFile(const FString& InVideoFilePath, const FString& InDiscoveryExpression) const;

    /** Returns true if the file extension is a video format OBS can produce. */
    static bool IsOBSVideoFile(const FString& InFilePath);

    /**
     * Convert a FilenameFormat (NamingTokens syntax) to a TakeDiscoveryExpression.
     * E.g. "{slate}-{take}" -> "<Slate>-<Take>", "{slate}_{take}_{yyyy}" -> "<Slate>_<Take>_<Any>"
     */
    static FString DeriveDiscoveryExpression(const FString& InFilenameFormat);

    // ----------------------------------------------------------------
    // Auth helpers
    // ----------------------------------------------------------------

    /** Compute the OBS WebSocket v5 auth response string. */
    static FString ComputeAuthResponse(const FString& InPassword, const FString& InChallenge, const FString& InSalt);

    // ----------------------------------------------------------------
    // State
    // ----------------------------------------------------------------

    ELiveLinkDeviceConnectionStatus ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;

    TSharedPtr<IWebSocket> WebSocket;

    /** True once OBS has accepted our Identify message (op 2 Identified received). */
    bool bIdentified = false;

    /** Current recording state as reported by OBS events. */
    bool bIsRecording = false;

    /** Counter for generating unique request IDs. */
    uint32 RequestCounter = 0;

    /** True when Disconnect() was called deliberately; suppresses the auto-reconnect. */
    bool bIntentionalDisconnect = false;

    /** True when OBS rejected our authentication (close code 4009); suppresses reconnect and shows an error. */
    bool bAuthFailed = false;

    /** True when OBS requires authentication but the password field is blank. */
    bool bPasswordRequired = false;

    /** True when a connection setting changed while connected; triggers a reconnect once the current disconnect completes. */
    bool bReconnectAfterSettingsChange = false;

    /** Handle for the pending reconnect ticker. Valid only while a reconnect is scheduled. */
    FTSTicker::FDelegateHandle ReconnectTickerHandle;

    /** Seconds to wait before attempting to reconnect after an unexpected disconnect. */
    static constexpr float ReconnectDelay = 5.f;

    /** Pending response callbacks keyed by request ID. */
    TMap<FString, FResponseCallback> PendingCallbacks;

    /** Cached OBS record directory, queried via GetRecordDirectory after identification. */
    FString CachedRecordDirectory;

    /**
     * The user's original FilenameFormatting value from the OBS profile, captured at connect time.
     * Restored after each Hub-driven recording so manual OBS recordings use the user's own format.
     */
    FString OriginalFilenameFormatting;

    /** True once OriginalFilenameFormatting has been successfully read from OBS. */
    bool bOriginalFilenameCached = false;

    /** Maps TakeId -> full file path on disk. Guarded by FullTakePathsMutex. */
    TMap<int32, FString> FullTakePaths;

    /** Protects FullTakePaths — accessed from game thread and background ingest tasks. */
    mutable FCriticalSection FullTakePathsMutex;

    /** Slate name captured at StartRecording time (for accurate auto-add). */
    FString RecordingSlateName;

    /** Take number captured at StartRecording time (for accurate auto-add). */
    int32 RecordingTakeNumber = INDEX_NONE;
};
