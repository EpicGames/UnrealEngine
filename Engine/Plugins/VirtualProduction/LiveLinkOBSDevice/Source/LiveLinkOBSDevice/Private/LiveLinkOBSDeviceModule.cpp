// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/LiveLinkOBSDeviceSettingsCustomization.h"
#include "Devices/LiveLinkOBSDevice.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"


class FLiveLinkOBSDeviceModule : public IModuleInterface
{
public:
    virtual void StartupModule() override
    {
        FPropertyEditorModule& PropertyModule =
            FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

        PropertyModule.RegisterCustomClassLayout(
            ULiveLinkOBSDeviceSettings::StaticClass()->GetFName(),
            FOnGetDetailCustomizationInstance::CreateStatic(
                &FLiveLinkOBSDeviceSettingsCustomization::MakeInstance));

        PropertyModule.NotifyCustomizationModuleChanged();
    }

    virtual void ShutdownModule() override
    {
        if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
        {
            FPropertyEditorModule& PropertyModule =
                FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

            PropertyModule.UnregisterCustomClassLayout(
                ULiveLinkOBSDeviceSettings::StaticClass()->GetFName());
        }
    }
};

IMPLEMENT_MODULE(FLiveLinkOBSDeviceModule, LiveLinkOBSDevice);
